{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Epic Stage App",
	"Description": "Enables remote connections from the Epic Stage App",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "EpicStageApp",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "nDisplay",
			"Enabled": true
		},
		{
			"Name": "nDisplayModularFeatures",
			"Enabled": true
		},
		{
			"Name": "DiscoveryBeaconReceiver",
			"Enabled": true
		}
	]
}