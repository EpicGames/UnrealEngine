// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EpicStageApp : ModuleRules
{
	public EpicStageApp(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Serialization"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DeveloperSettings",
				"DiscoveryBeaconReceiver",
				"DisplayCluster",
				"DisplayClusterLightCardEditorShaders",
				"DisplayClusterLightCardExtender",
				"DisplayClusterScenePreview",
				"Engine",
				"ImageWrapper",
				"Networking",
				"RemoteControl",
				"RemoteControlCommon",
				"RHI",
				"Sockets",
				"WebRemoteControl"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DisplayClusterLightCardEditor",
					"UnrealEd",
				}
			);
		}
	}
}
