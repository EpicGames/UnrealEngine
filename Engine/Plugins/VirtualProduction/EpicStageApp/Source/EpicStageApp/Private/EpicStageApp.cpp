// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicStageApp.h"
#include "IWebRemoteControlModule.h"

IMPLEMENT_MODULE(FEpicStageAppModule, EpicStageApp);

void FEpicStageAppModule::StartupModule()
{
	IWebRemoteControlModule& WebRemoteControl = FModuleManager::LoadModuleChecked<IWebRemoteControlModule>("WebRemoteControl");
	RouteHandler.RegisterRoutes(WebRemoteControl);

	if (!IsRunningCommandlet())
	{
		if (WebRemoteControl.IsWebSocketServerRunning())
		{
			StageAppBeaconReceiver.Startup();
		}
		else
		{
			// Wait for the WebSocket server to start
			WebSocketServerStartedDelegate = WebRemoteControl.OnWebSocketServerStarted().AddLambda([this](uint32)
			{
				StageAppBeaconReceiver.Startup();
			});
		}
	}
}

void FEpicStageAppModule::ShutdownModule()
{
	if (IWebRemoteControlModule* WebRemoteControl = FModuleManager::GetModulePtr<IWebRemoteControlModule>("WebRemoteControl"))
	{
		RouteHandler.UnregisterRoutes(*WebRemoteControl);

		if (WebSocketServerStartedDelegate.IsValid())
		{
			WebRemoteControl->OnWebSocketServerStarted().Remove(WebSocketServerStartedDelegate);
		}
	}

	StageAppBeaconReceiver.Shutdown();
}
