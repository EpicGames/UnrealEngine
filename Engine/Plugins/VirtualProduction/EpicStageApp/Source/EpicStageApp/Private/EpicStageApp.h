// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StageAppBeaconReceiver.h"
#include "StageAppRouteHandler.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FEpicStageAppModule : public IModuleInterface
{
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
	
	/** Handler for WebSocket routes. */
	FStageAppRouteHandler RouteHandler;

private:
	/** Receiver that responds to beacon messages from the stage app. */
	FStageAppBeaconReceiver StageAppBeaconReceiver;

	/** Delegate for when the websocket server starts. */
	FDelegateHandle WebSocketServerStartedDelegate;
};
