// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageAppBeaconReceiver.h"

#include "Common/UdpSocketBuilder.h"
#include "IRemoteControlModule.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "RemoteControlSettings.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "StageAppSettings.h"

namespace StageAppBeaconReceiverConstants
{
	const uint8 ProtocolVersion = 0;
	const TArray<uint8> ProtocolIdentifier = { 'E', 'S', '@', 'p' };
}

FStageAppBeaconReceiver::FStageAppBeaconReceiver()
	: FDiscoveryBeaconReceiver(
		TEXT("StageAppBeaconResponder"),
		StageAppBeaconReceiverConstants::ProtocolIdentifier,
		StageAppBeaconReceiverConstants::ProtocolVersion
	)
{
}

void FStageAppBeaconReceiver::Startup()
{
	WebsocketPort = GetDefault<URemoteControlSettings>()->RemoteControlWebSocketServerPort;

	FDiscoveryBeaconReceiver::Startup();
}

bool FStageAppBeaconReceiver::GetDiscoveryAddress(FIPv4Address& OutAddress) const
{
	const UStageAppSettings& Settings = *GetDefault<UStageAppSettings>();
	FIPv4Address DiscoveryAddress;
	if (!FIPv4Address::Parse(Settings.DiscoveryEndpoint, DiscoveryAddress))
	{
		UE_LOGF(LogRemoteControl, Error, "Failed to parse Stage App discovery endpoint address \"%ls\"", *Settings.DiscoveryEndpoint);
		return false;
	}

	OutAddress = DiscoveryAddress;
	return true;
}

int32 FStageAppBeaconReceiver::GetDiscoveryPort() const
{
	const UStageAppSettings& Settings = *GetDefault<UStageAppSettings>();
	return Settings.DiscoveryPort;
}

bool FStageAppBeaconReceiver::MakeBeaconResponse(uint8 BeaconProtocolVersion, FArrayReader& InMessageData, FArrayWriter& OutResponseData) const
{
	const FString FriendlyName = GetFriendlyName();

	{
		OutResponseData << (uint32&)WebsocketPort;
		OutResponseData << (FString&)FriendlyName;
	}

	return true;
}

FString FStageAppBeaconReceiver::GetFriendlyName() const
{
	FString FriendlyName;

	if (FParse::Value(FCommandLine::Get(), TEXT("-StageFriendlyName="), FriendlyName))
	{
		return FriendlyName;
	}

	if (FParse::Value(FCommandLine::Get(), TEXT("-CONCERTDISPLAYNAME="), FriendlyName))
	{
		return FriendlyName;
	}

	return FApp::GetSessionOwner();
}
