// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiscoveryBeaconReceiver.h"

/**
 * Receives beacon messages from the Unreal Stage App and replies with connection information.
 * This allows the app to detect compatible Unreal instances on the local network and list them for the user.
 */
class FStageAppBeaconReceiver
	: public FDiscoveryBeaconReceiver
{
public:
	FStageAppBeaconReceiver();
	virtual ~FStageAppBeaconReceiver() {};

	//~ Begin FDiscoveryBeaconReceiver implementation
public:
	virtual void Startup() override;

protected:
	virtual bool GetDiscoveryAddress(FIPv4Address& OutAddress) const override;
	virtual int32 GetDiscoveryPort() const override;
	virtual bool MakeBeaconResponse(uint8 BeaconProtocolVersion, FArrayReader& InMessageData, FArrayWriter& OutResponseData) const override;
	//~ End FDiscoveryBeaconReceiver implementation

private:
	/** Get the name to report to apps searching for the engine. */
	FString GetFriendlyName() const;

	/** The websocket port with which to reply. */
	uint32 WebsocketPort;
};