// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageAppLibrary.h"
#include "StageAppVersion.h"
#include "RemoteControlSettings.h"

FString UStageAppFunctionLibrary::GetAPIVersion()
{
	return FString::Printf(TEXT("%u.%u.%u"),
		FEpicStageAppAPIVersion::Major,
		FEpicStageAppAPIVersion::Minor,
		FEpicStageAppAPIVersion::Patch);
}

int32 UStageAppFunctionLibrary::GetRemoteControlWebInterfacePort()
{
	return static_cast<int32>(GetDefault<URemoteControlSettings>()->RemoteControlWebInterfacePort);
}