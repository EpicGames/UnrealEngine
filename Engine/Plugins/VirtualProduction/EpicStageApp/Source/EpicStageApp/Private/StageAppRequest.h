// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "CoreMinimal.h"
#include "IDisplayClusterScenePreview.h"
#include "RemoteControlWebsocketRoute.h"

#include "StageAppRequest.generated.h"

/** Type of preview render to perform, exposed as a UENUM. Corresponds to EDisplayClusterMeshProjectionOutput. */
UENUM()
enum class ERCWebSocketNDisplayPreviewRenderType : uint8
{
	Color,
	Normals
};

/** Type of projection to use for a preview render, exposed as a UENUM. Corresponds to EDisplayClusterMeshProjectionType. */
UENUM()
enum class ERCWebSocketNDisplayPreviewRenderProjectionType : uint8
{
	Perspective,
	Azimuthal,
	Orthographic,
	UV
};

/** Preview renderer settings exposed to WebSocket clients. */
USTRUCT()
struct FRCWebSocketNDisplayPreviewRendererSettings
{
	GENERATED_BODY()

	/**
	 * The type of render to perform.
	 */
	UPROPERTY()
	ERCWebSocketNDisplayPreviewRenderType RenderType = ERCWebSocketNDisplayPreviewRenderType::Color;

	/**
	 * The type of projection to use.
	 */
	UPROPERTY()
	ERCWebSocketNDisplayPreviewRenderProjectionType ProjectionType = ERCWebSocketNDisplayPreviewRenderProjectionType::Azimuthal;

	/**
	 * The resolution of the image to render.
	 */
	UPROPERTY()
	FIntPoint Resolution = FIntPoint(1024, 1024);

	/**
	 * The preview camera's field of view (both horizontal and vertical) in degrees.
	 */
	UPROPERTY()
	float FOV = 130.0f;

	/**
	 * The preview camera's Euler rotation relative to the camera's actual rotation in the scene.
	 */
	UPROPERTY()
	FRotator Rotation = FRotator(90, 0, 0);

	/**
	 * The quality of the JPEG to send back to the requesting client, in range 50-100.
	 */
	UPROPERTY()
	int32 JpegQuality = 50;

	/**
	 * If true, include a list of projected positions within the preview render for each rendered actor when responding to a render request.
	 */
	UPROPERTY()
	bool IncludeActorPositions = false;
};

/**
 * Holds a request made via websocket to create an nDisplay preview renderer.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewRendererCreateBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewRendererCreateBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The path of the root DisplayCluster actor to preview. This may be empty, in which case you can set the root actor later.
	 */
	UPROPERTY()
	FString RootActorPath = "";

	/**
	 * Initial settings for the renderer.
	 */
	UPROPERTY()
	FRCWebSocketNDisplayPreviewRendererSettings Settings;
};

/**
 * Holds a request made via websocket to change the root DisplayCluster actor of an nDisplay preview renderer.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewRendererSetRootBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewRendererSetRootBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * The path of the root DisplayCluster actor to preview.
	 */
	UPROPERTY()
	FString RootActorPath = "";
};

/**
 * Holds a request made via websocket to destroy an nDisplay preview renderer.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewRendererDestroyBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewRendererDestroyBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;
};

/**
 * Change a preview renderer's settings for future renders.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewRendererConfigureBody : public FRCRequest
{
	GENERATED_BODY()
		
	/**
	 * The ID of the renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * Settings to use for future renders.
	 */
	UPROPERTY()
	FRCWebSocketNDisplayPreviewRendererSettings Settings;
};

/**
 * Holds a request made via websocket to render a preview of a display cluster.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewRenderBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewRenderBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;
};

/**
 * Holds a request made via websocket to start dragging one or more actors.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewActorDragBeginBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewActorDragBeginBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the preview renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * Paths of the actors that will be included in this drag action.
	 */
	UPROPERTY()
	TArray<FString> Actors;

	/**
	 * The actor to use as the origin point for the drag.
	 */
	UPROPERTY()
	FString PrimaryActor;

	/**
	 * The sequence number of this change. The highest sequence number received from this client will be
	 * sent back to the client in future position updates.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;
};


/**
 * Holds a request made via websocket to move the actors that are currently being dragged for a preview renderer.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewActorDragMoveBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewActorDragMoveBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the preview renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * The position of the drag cursor in coordinates normalized to the size of the preview image.
	 */
	UPROPERTY()
	FVector2D DragPosition = FVector2D::ZeroVector;

	/**
	 * The sequence number of this change. The highest sequence number received from this client will be
	 * sent back to the client in future position updates.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;
};

/**
 * Holds a request made via websocket to stop dragging actors for a preview renderer.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewActorDragEndBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewActorDragEndBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the preview renderer returned when it was created.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * The final position of the drag cursor in coordinates normalized to the size of the preview image.
	 */
	UPROPERTY()
	FVector2D DragPosition = FVector2D::ZeroVector;

	/**
	 * The sequence number of this change. The highest sequence number received from this client will be
	 * sent back to the client in future position updates.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;
};

/**
 * Holds a request made via websocket to create an actor, optionally positioned relative to the previewed area.
 */
USTRUCT()
struct FRCWebSocketNDisplayPreviewActorCreateBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayPreviewActorCreateBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the preview renderer to use when positioning this light card.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * The name of the actor to spawn. A number will be appended if this conflicts with another name.
	 */
	UPROPERTY()
	FString ActorName = "";

	/**
	 * The name of the class of the actor to spawn.
	 */
	UPROPERTY()
	FString ActorClass = "";

	/**
	 * The path of the template to use for the lightcard.
	 * If empty, a lightcard will be created using the default template.
	 * If "None", a lightcard will be created with default settings regardless of whether there's a default template.
	 */
	UPROPERTY()
	FString TemplatePath = "";

	/**
	 * If true, override the default/template position for the lightcard with the provided Position.
	 * Otherwise, use the template's position, or if no template, create at viewport center (0.5, 0.5).
	 */
	UPROPERTY()
	bool OverridePosition = false;

	/**
	 * If OverridePosition is true and PreviewRendererId points to a valid preview renderer, place the
	 * lightcard at this normalized viewport coordinate (in range [0, 1]).
	 */
	UPROPERTY()
	FVector2D Position = FVector2D::ZeroVector;

	/**
	 * If true, override the default/template color for the actor if it's a lightcard.
	 */
	UPROPERTY()
	bool OverrideColor = false;

	/**
	 * If OverrideColor is true and PreviewRendererId points to a valid preview renderer, use this color when creating
	 * a lightcard.
	 */
	UPROPERTY()
	FLinearColor Color = FLinearColor::White;

	/**
	 * Whether this actor should be created as a flag if it's a lightcard.
	 */
	UPROPERTY()
	bool bIsLightCardFlag = false;

	/**
	 * An optional number that will be passed back in the RequestedActorsCreated response to tell apart
	 * the results of multiple requests.
	 */
	UPROPERTY()
	int32 RequestId = -1;
};

/**
 * Holds a request made via websocket to duplicate one or more actors.
 */
USTRUCT()
struct FRCWebSocketNDisplayActorDuplicateBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketNDisplayActorDuplicateBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The list of paths of actors to duplicate.
	 */
	UPROPERTY()
	TArray<FString> Actors;

	/**
	 * An optional number that will be passed back in the RequestedActorsCreated response to tell apart
	 * the results of multiple requests.
	 */
	UPROPERTY()
	int32 RequestId = -1;
};

/**
 * Holds a request made via websocket to create an object whose lifetime will be bound to the lifetime
 * of the requesting client's connection.
 */
USTRUCT()
struct FRCWebSocketStageAppClientObjectCreateBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketStageAppClientObjectCreateBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The class of the object to create.
	 */
	UPROPERTY()
	TSubclassOf<UObject> ObjectClass;

	/**
	 * An optional number that will be passed back in the ClientObjectCreated response to tell apart
	 * the results of multiple requests.
	 */
	UPROPERTY()
	int32 RequestId = -1;
};

/**
 * Holds a request made via websocket to destroy an object whose lifetime is bound to the lifetime
 * of the requesting client's connection.
 */
USTRUCT()
struct FRCWebSocketStageAppClientObjectDestroyBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketStageAppClientObjectDestroyBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The path of the object. Must be an object created using the stageapp.clientobject.create route.
	 */
	UPROPERTY()
	FString ObjectPath;

	/**
	 * An optional number that will be passed back in the RequestedClientObjectCreated response to tell
	 * apart the results of multiple requests.
	 */
	UPROPERTY()
	int32 RequestId = -1;
};
