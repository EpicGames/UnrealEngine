// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "StageAppResponse.generated.h"

/**
 * Event sent to a client that created a renderer.
 */
USTRUCT()
struct FRCPreviewRendererCreatedEvent
{
	GENERATED_BODY()

	FRCPreviewRendererCreatedEvent()
	: Type(TEXT("PreviewRendererCreated"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The ID of the new renderer
	 */
	UPROPERTY()
	int32 RendererId = -1;
};

USTRUCT()
struct FRCPreviewRenderCompletedEventActorPosition
{
	GENERATED_BODY()

	/** The actor's path. */
	UPROPERTY()
	FString Path;

	/** The actor's position on the preview texture in coordinates normalized to the size of the preview render's texture. */
	UPROPERTY()
	FVector2D Position = FVector2D::ZeroVector;
};

/**
 * Event sent to a client that created a renderer.
 */
USTRUCT()
struct FRCPreviewRenderCompletedEvent
{
	GENERATED_BODY()

	FRCPreviewRenderCompletedEvent()
	: Type(TEXT("PreviewRenderCompleted"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The ID of the renderer that created the image.
	 */
	UPROPERTY()
	int32 RendererId = -1;

	/**
	 * Base64 encoded JPEG data of the preview render.
	 */
	UPROPERTY()
	FString ImageBase64 = "";

	/**
	 * The resolution of the image included with this message.
	 */
	UPROPERTY()
	FIntPoint Resolution = FIntPoint(1024, 1024);

	/**
	 * A list of projected positions within the preview render for each rendered actor.
	 */
	UPROPERTY()
	TArray<FRCPreviewRenderCompletedEventActorPosition> ActorPositions;

	/**
	 * The last sequence number that the client sent during a stage actor drag operation on this renderer.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;

	/**
	 * If true, real-time preview rendering is disabled in the engine, which may prevent nDisplay previews and stage actor positions from updating correctly.
	 */
	UPROPERTY()
	bool IsRealTimeDisabled = false;
};

/**
 * Event sent to a client when a drag operation is cancelled by the engine.
 */
USTRUCT()
struct FRCActorDragCancelled
{
	GENERATED_BODY()

	FRCActorDragCancelled()
	: Type(TEXT("ActorDragCancelled"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The ID of the renderer associated with the drag operation.
	 */
	UPROPERTY()
	int32 RendererId = -1;
};

/**
 * Event sent to a client when light cards were created at the request of the client.
 */
USTRUCT()
struct FRCRequestedActorsCreated
{
	GENERATED_BODY()

	FRCRequestedActorsCreated()
	: Type(TEXT("RequestedActorsCreated"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The list of newly-created actors' paths.
	 */
	UPROPERTY()
	TArray<FString> ActorPaths;

	/**
	 * The ID passed when the actors were created, or -1 if no ID was provided.
	 */
	UPROPERTY()
	int32 RequestId = -1;
};

/**
 * Event sent to a client when a client-bound object was created at the request of the client.
 */
USTRUCT()
struct FRCRequestedClientObjectCreated
{
	GENERATED_BODY()

	FRCRequestedClientObjectCreated()
	: Type(TEXT("RequestedClientObjectCreated"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The list of newly-created object's path.
	 */
	UPROPERTY()
	FString ObjectPath;

	/**
	 * The ID passed when the actors were created, or -1 if no ID was provided.
	 */
	UPROPERTY()
	int32 RequestId = -1;
};
