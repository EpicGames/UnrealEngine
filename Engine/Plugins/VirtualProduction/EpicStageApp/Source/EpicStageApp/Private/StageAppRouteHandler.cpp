// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageAppRouteHandler.h"

#include "CanvasTypes.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterLightCardEditorHelper.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterRootActorContainers.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IDisplayClusterScenePreview.h"
#include "IImageWrapperModule.h"
#include "IRemoteControlModule.h"
#include "Misc/Base64.h"
#include "Modules/ModuleManager.h"
#include "RemoteControlWebsocketRoute.h"
#include "StageAppResponse.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "WebRemoteControlUtils.h"
#include "StageActor/DisplayClusterStageActorTemplate.h"
#include "StageActor/DisplayClusterWeakStageActorPtr.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/TransBuffer.h"
#include "IDisplayClusterLightCardEditor.h"
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "StageAppRouteHandler"

static TAutoConsoleVariable<float> CVarStageAppDragTimeoutCheckInterval(
	TEXT("StageApp.DragTimeoutCheckInterval"),
	2.f,
	TEXT("The interval at which to check whether a drag operation is still active, and if not, time it out.")
);

namespace StageAppRouteHandlerUtils
{
	/**
	 * Convert from a projection type provided over WebSocket to the values needed internally.
	 */
	void GetProjectionSettingsFromWebSocketType(
		ERCWebSocketNDisplayPreviewRenderProjectionType InProjectionType,
		EDisplayClusterMeshProjectionType& OutProjectionType,
		bool& bOutIsOrthographic)
	{
		switch (InProjectionType)
		{
		case ERCWebSocketNDisplayPreviewRenderProjectionType::Azimuthal:
			OutProjectionType = EDisplayClusterMeshProjectionType::Azimuthal;
			bOutIsOrthographic = false;
			break;

		case ERCWebSocketNDisplayPreviewRenderProjectionType::Perspective:
			OutProjectionType = EDisplayClusterMeshProjectionType::Linear;
			bOutIsOrthographic = false;
			break;

		case ERCWebSocketNDisplayPreviewRenderProjectionType::Orthographic:
			OutProjectionType = EDisplayClusterMeshProjectionType::Linear;
			bOutIsOrthographic = true;
			break;

		case ERCWebSocketNDisplayPreviewRenderProjectionType::UV:
			OutProjectionType = EDisplayClusterMeshProjectionType::UV;
			bOutIsOrthographic = true;
			break;

		default:
			checkf(false, TEXT("Unknown projection type %d"), EnumToUnderlyingType(InProjectionType));
		}
	}

	/**
	 * Convert from a preview render type provided over WebSocket to the internal value used for the preview renderer.
	 */
	EDisplayClusterMeshProjectionOutput GetInternalPreviewRenderType(ERCWebSocketNDisplayPreviewRenderType InRenderType)
	{
		switch (InRenderType)
		{
		case ERCWebSocketNDisplayPreviewRenderType::Color:
			return EDisplayClusterMeshProjectionOutput::Color;

		case ERCWebSocketNDisplayPreviewRenderType::Normals:
			return EDisplayClusterMeshProjectionOutput::Normals;

		default:
			checkf(false, TEXT("Unknown projection type %d"), EnumToUnderlyingType(InRenderType));
			return EDisplayClusterMeshProjectionOutput::Color;
		}
	}

	/// Given a light card, find the ADisplayClusterRootActor to which it belongs (or null if none).
	ADisplayClusterRootActor* GetRootActorForLightCard(AActor* InActor)
	{
		UWorld* World = InActor->GetWorld();
		if (!World)
		{
			return nullptr;
		}

		for (TActorIterator<ADisplayClusterRootActor> ActorIterator(World); ActorIterator; ++ActorIterator)
		{
			ADisplayClusterRootActor* RootActor = *ActorIterator;
			if (!RootActor)
			{
				continue;
			}

			const UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData();
			if (!ConfigData)
			{
				continue;
			}

			const FDisplayClusterConfigurationICVFX_VisibilityList& RootActorLightCards = ConfigData->StageSettings.Lightcard.ShowOnlyList;

			// Check if it's in the root actor's layers
			for (const FActorLayer& ActorLayer : RootActorLightCards.ActorLayers)
			{
				if (InActor->Layers.Contains(ActorLayer.Name))
				{
					return RootActor;
				}
			}

			// Check if it's directly on the root actor
			if (RootActorLightCards.Actors.ContainsByPredicate([InActor](const TSoftObjectPtr<AActor>& Actor)
				{
					return InActor == Actor.Get();
				}))
			{
				return RootActor;
			}
		}

		return nullptr;
	}
}


//////////////////////////////////////////////////////////////////////////
// FPerRendererData

FStageAppRouteHandler::FPerRendererData::FPerRendererData(int32 RendererId)
	: RendererId(RendererId)
{
}

FDisplayClusterLightCardEditorHelper& FStageAppRouteHandler::FPerRendererData::GetLightCardHelper()
{
	if (!LightCardHelper.IsValid())
	{
		LightCardHelper = MakeShared<FDisplayClusterLightCardEditorHelper>(RendererId);
		UpdateLightCardHelperSettings();
	}

	return *LightCardHelper;
}

const FRCWebSocketNDisplayPreviewRendererSettings& FStageAppRouteHandler::FPerRendererData::GetPreviewSettings() const
{
	return PreviewSettings;
}

void FStageAppRouteHandler::FPerRendererData::SetPreviewSettings(const FRCWebSocketNDisplayPreviewRendererSettings& NewSettings)
{
	PreviewSettings = NewSettings;
	UpdateLightCardHelperSettings();
}

bool FStageAppRouteHandler::FPerRendererData::GetSceneViewInitOptions(FSceneViewInitOptions& ViewInitOptions, bool bApplyActorRotation)
{
	ADisplayClusterRootActor* RootActorProxy = GetRootActorProxy();
	if (!RootActorProxy)
	{
		return false;
	}

	USceneComponent* ViewOriginComponent = RootActorProxy->GetCommonViewPoint();
	if (!ViewOriginComponent)
	{
		return false;
	}

	// In UV mode, lock the location and rotation
	const bool bIsUVMode = GetLightCardHelper().GetProjectionMode() == EDisplayClusterMeshProjectionType::UV;
	const FVector Location = bIsUVMode ? FVector::ZeroVector : ViewOriginComponent->GetComponentLocation();
	FRotator Rotation;
	
	if (bIsUVMode)
	{
		Rotation = FVector::ForwardVector.Rotation();
	}
	else
	{
		if (bApplyActorRotation)
		{
			Rotation = FRotator(RootActorProxy->GetActorRotation().Quaternion() * PreviewSettings.Rotation.Quaternion());
		}
		else
		{
			Rotation = PreviewSettings.Rotation;
		}
	}

	GetLightCardHelper().GetSceneViewInitOptions(
		ViewInitOptions,
		PreviewSettings.FOV,
		PreviewSettings.Resolution,
		Location,
		Rotation
	);

	return true;
}

void FStageAppRouteHandler::FPerRendererData::UpdateDragSequenceNumber(int64 ReceivedSequenceNumber)
{
	if (ReceivedSequenceNumber > SequenceNumber)
	{
		SequenceNumber = ReceivedSequenceNumber;
	}
}

int64 FStageAppRouteHandler::FPerRendererData::GetDragSequenceNumber() const
{
	return SequenceNumber;
}

ADisplayClusterRootActor* FStageAppRouteHandler::FPerRendererData::GetRootActor() const
{
	return IDisplayClusterScenePreview::Get().GetRendererRootActor(RendererId);
}

void FStageAppRouteHandler::FPerRendererData::UpdateLightCardHelperSettings()
{
	if (FDisplayClusterLightCardEditorHelper* LightCardHelperPtr = LightCardHelper.Get())
	{
		EDisplayClusterMeshProjectionType ProjectionType;
		bool bIsOrthographic;

		StageAppRouteHandlerUtils::GetProjectionSettingsFromWebSocketType(PreviewSettings.ProjectionType, ProjectionType, bIsOrthographic);

		LightCardHelperPtr->SetProjectionMode(ProjectionType);
		LightCardHelperPtr->SetIsOrthographic(bIsOrthographic);
	}
}

//////////////////////////////////////////////////////////////////////////
// FStageAppRouteHandler

void FStageAppRouteHandler::RegisterRoutes(IWebRemoteControlModule& WebRemoteControl)
{
	RemoteControlModule = &WebRemoteControl;
	RemoteControlModule->OnWebSocketConnectionClosed().AddRaw(this, &FStageAppRouteHandler::HandleClientDisconnected);

	ImageWrapperModule = &FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

	if (GEngine)
	{
		RegisterEngineEvents();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStageAppRouteHandler::RegisterEngineEvents);
	}

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Create a preview renderer for a specific nDisplay cluster"),
		TEXT("ndisplay.preview.renderer.create"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererCreate)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Set the root DisplayCluster actor for an nDisplay preview renderer"),
		TEXT("ndisplay.preview.renderer.setroot"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererSetRoot)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Destroy an nDisplay preview renderer"),
		TEXT("ndisplay.preview.renderer.destroy"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererDestroy)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Change the configuration of an nDisplay preview renderer. Old values will be retained for any properties not explicitly included."),
		TEXT("ndisplay.preview.renderer.configure"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererConfigure)
		));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Request a preview render from an nDisplay preview renderer"),
		TEXT("ndisplay.preview.render"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRender)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Start dragging actors relative to a projected preview"),
		TEXT("ndisplay.preview.actor.drag.begin"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorDragBegin)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Move the actors that are currently being dragged"),
		TEXT("ndisplay.preview.actor.drag.move"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorDragMove)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Finish dragging the actors that are currently being dragged"),
		TEXT("ndisplay.preview.actor.drag.end"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorDragEnd)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Create an actor positioned relative to the preview renderer's viewport"),
		TEXT("ndisplay.preview.actor.create"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorCreate)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Duplicate existing actors"),
		TEXT("stageapp.actors.duplicate"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketStageAppActorsDuplicate)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Create an object that will be kept alive until the client disconnects"),
		TEXT("stageapp.clientobject.create"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketStageAppClientObjectCreate)
	));

	RegisterRoute(MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Destroy an object that was created using the stageapp.clientobject.create route"),
		TEXT("stageapp.clientobject.destroy"),
		FWebSocketMessageDelegate::CreateRaw(this, &FStageAppRouteHandler::HandleWebSocketStageAppClientObjectDestroy)
	));
}

void FStageAppRouteHandler::RegisterEngineEvents()
{
#if WITH_EDITOR
	if (GEditor)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransBuffer->OnTransactionStateChanged().AddRaw(this, &FStageAppRouteHandler::OnTransactionStateChanged);
		}
	}
#endif
}

void FStageAppRouteHandler::UnregisterRoutes(IWebRemoteControlModule& WebRemoteControl)
{
	if (RemoteControlModule == &WebRemoteControl)
	{
		for (const TUniquePtr<FRemoteControlWebsocketRoute>& Route : Routes)
		{
			WebRemoteControl.UnregisterWebsocketRoute(*Route);
		}
	}

	if (DragTimeoutTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(DragTimeoutTickerHandle);
		DragTimeoutTickerHandle.Reset();
	}

	UnregisterEngineEvents();
}

void FStageAppRouteHandler::UnregisterEngineEvents()
{
#if WITH_EDITOR
	if (GEditor)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransBuffer->OnTransactionStateChanged().RemoveAll(this);
		}
	}
#endif
}

void FStageAppRouteHandler::RegisterRoute(TUniquePtr<FRemoteControlWebsocketRoute> Route)
{
	checkSlow(RemoteControlModule);

	RemoteControlModule->RegisterWebsocketRoute(*Route);
	Routes.Emplace(MoveTemp(Route));
}

ADisplayClusterRootActor* FStageAppRouteHandler::FPerRendererData::GetRootActorProxy() const
{
	// In this application we will always use RootActorProxy, so the function below will always return a proxy.
	// (see EDisplayClusterScenePreviewFlags::UseRootActorProxy flag)
	return IDisplayClusterScenePreview::Get().GetRendererRootActorOrProxy(RendererId);
}

void FStageAppRouteHandler::SetRendererRootActorPath(const int32 RendererId, const FString& RootActorPath)
{
	// Use custom settings for root actor.
	FDisplayClusterRootActorPropertyOverrides PropertyOverrides;
	{
		PropertyOverrides.bPreviewICVFXFrustums = false;
		PropertyOverrides.bEnablePreviewTechvis = false;
		PropertyOverrides.bPreviewEnableOverlayMaterial = false;
		PropertyOverrides.bFreezePreviewRender = false;

		PropertyOverrides.bPreviewEnablePostProcess = true;
		PropertyOverrides.bPreviewEnable = true;

		// Can render a DCRA preview without the HoldoutComposite plugin.
		PropertyOverrides.bPreviewEnableHoldoutComposite = false;

		PropertyOverrides.PreviewSetttingsSource = EDisplayClusterConfigurationRootActorPreviewSettingsSource::RootActor;
	}

	IDisplayClusterScenePreview::Get().SetRendererRootActorPath(RendererId, RootActorPath, PropertyOverrides,
		// When we use preview rendering for an external request with settings different from those used in the scene,
		// need to create a new DCRA actor and use it as a proxy for custom rendering.
		EDisplayClusterScenePreviewFlags::UseRootActorProxy
		// Move the RootActorProxy to the same position as in the scene to match the position of the StageActors in world space.
		| EDisplayClusterScenePreviewFlags::ProxyFollowSceneRootActor
		// Also proxy must render preview each frame
		| EDisplayClusterScenePreviewFlags::ProxyTickPreviewRenderer
		// Automatically update the renderer with stage actors belonging to the root actor in scene.
		| EDisplayClusterScenePreviewFlags::AutoUpdateStageActors
	);
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererCreate(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewRendererCreateBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	IDisplayClusterScenePreview& PreviewModule = IDisplayClusterScenePreview::Get();

	const int32 RendererId = PreviewModule.CreateRenderer();
	PerRendererDataMapsByClientId.FindOrAdd(WebSocketMessage.ClientId).Emplace(RendererId, RendererId);

	if (!Body.RootActorPath.IsEmpty())
	{
		SetRendererRootActorPath(RendererId, Body.RootActorPath);
	}

	ChangePreviewRendererSettings(WebSocketMessage.ClientId, RendererId, Body.Settings);

	FRCPreviewRendererCreatedEvent Event;
	Event.RendererId = RendererId;

	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(Event, Payload);

	RemoteControlModule->SendWebsocketMessage(WebSocketMessage.ClientId, Payload);
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererSetRoot(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewRendererSetRootBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	SetRendererRootActorPath(Body.RendererId, Body.RootActorPath);
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererConfigure(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewRendererConfigureBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FPerRendererData* PerRendererData = GetClientPerRendererData(WebSocketMessage.ClientId, Body.RendererId);
	if (PerRendererData)
	{
		// If we have existing settings for this renderer, copy its old settings and deserialize again over them. This lets us retain any
		// values that weren't explicitly included in the WebSocket message.
		Body.Settings = PerRendererData->GetPreviewSettings();
		if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
		{
			return;
		}
	}

	ChangePreviewRendererSettings(WebSocketMessage.ClientId, Body.RendererId, Body.Settings);
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRendererDestroy(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewRendererDestroyBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	if (TClientIdToPerRendererDataMap* PerRendererDataMap = PerRendererDataMapsByClientId.Find(WebSocketMessage.ClientId))
	{
		// Check that this client created the renderer
		if (PerRendererDataMap->Remove(Body.RendererId) > 0)
		{
			IDisplayClusterScenePreview::Get().DestroyRenderer(Body.RendererId);
		}
	}
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewRender(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	if (!ImageWrapperModule)
	{
		return;
	}

	FRCWebSocketNDisplayPreviewRenderBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FPerRendererData* PerRendererData = GetClientPerRendererData(WebSocketMessage.ClientId, Body.RendererId);
	if (!PerRendererData || PerRendererData->bIsRenderPending)
	{
		return;
	}

	const FRCWebSocketNDisplayPreviewRendererSettings& PreviewSettings = PerRendererData->GetPreviewSettings();
	const FIntPoint Resolution = PreviewSettings.Resolution;

	if (Resolution.GetMin() <= 0)
	{
		// Not a valid resolution, so skip this render
		return;
	}

	// Set up the render settings
	FDisplayClusterMeshProjectionRenderSettings RenderSettings;
	if (!PerRendererData->GetSceneViewInitOptions(RenderSettings.ViewInitOptions))
	{
		return;
	}

	RenderSettings.RenderType = StageAppRouteHandlerUtils::GetInternalPreviewRenderType(PreviewSettings.RenderType);
	RenderSettings.EngineShowFlags.SetSelectionOutline(false);
	
	EDisplayClusterMeshProjectionType ProjectionType;
	bool bIsOrthographic;

	StageAppRouteHandlerUtils::GetProjectionSettingsFromWebSocketType(PreviewSettings.ProjectionType, ProjectionType, bIsOrthographic);
	EngineShowFlagOrthographicOverride(!bIsOrthographic, RenderSettings.EngineShowFlags);

	const FDisplayClusterLightCardEditorHelper& Helper = PerRendererData->GetLightCardHelper();
	Helper.ConfigureRenderProjectionSettings(RenderSettings);
	RenderSettings.PrimitiveFilter.ShouldRenderPrimitiveDelegate = Helper.CreateDefaultShouldRenderPrimitiveFilter();
	RenderSettings.PrimitiveFilter.ShouldApplyProjectionDelegate = Helper.CreateDefaultShouldApplyProjectionToPrimitiveFilter();

	FSceneViewInitOptions ViewInitOptions = RenderSettings.ViewInitOptions;
	const FGuid ClientId = WebSocketMessage.ClientId;
	const int32 RendererId = Body.RendererId;
	const int32 JpegQuality = PreviewSettings.JpegQuality;
	const bool bIncludeActorPositions = PreviewSettings.IncludeActorPositions;

	PerRendererData->bIsRenderPending = true;

	// Render the image
	IDisplayClusterScenePreview::Get().RenderQueued(Body.RendererId, RenderSettings, Resolution, FRenderResultDelegate::CreateLambda(
		[this, ViewInitOptions, ClientId, RendererId, ProjectionType, Resolution, JpegQuality, bIncludeActorPositions](FRenderTarget* RenderTarget)
		{
			FPerRendererData* PerRendererData = GetClientPerRendererData(ClientId, RendererId);
			if (PerRendererData)
			{
				PerRendererData->bIsRenderPending = false;
			}

			if (!RenderTarget)
			{
				// Render failed
				return;
			}

			// Read image data
			TArray<FColor> PixelData;
			RenderTarget->ReadPixels(PixelData);
			FImageView ImageView(PixelData.GetData(), Resolution.X, Resolution.Y, 1, ERawImageFormat::BGRA8, EGammaSpace::Linear);

			// Convert to JPEG
			TArray64<uint8> JpegData;
			ImageWrapperModule->CompressImage(JpegData, EImageFormat::JPEG, ImageView, JpegQuality);

			// Encode to Base64
			FRCPreviewRenderCompletedEvent Event;
			Event.RendererId = RendererId;
			Event.ImageBase64 = FBase64::Encode(JpegData.GetData(), JpegData.Num());
			Event.Resolution = Resolution;
			Event.IsRealTimeDisabled = !IDisplayClusterScenePreview::Get().IsRealTimePreviewEnabled();

			if (PerRendererData)
			{
				Event.SequenceNumber = PerRendererData->GetDragSequenceNumber();

				// Add the list of actors, if requested
				if (bIncludeActorPositions)
				{
					TArray<AActor*> Actors;
					if (IDisplayClusterScenePreview::Get().GetActorsInRendererScene(RendererId, false, Actors))
					{
						const FSceneView View(ViewInitOptions);
						const bool bIsInUVMode = ProjectionType == EDisplayClusterMeshProjectionType::UV;

						for (const AActor* Actor : Actors)
						{
							if (!Actor)
							{
								continue;
							}

							FVector WorldPosition;
							if (const IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(Actor))
							{
								// Only send UV actors in UV mode, and only send non-UV actors in non-UV modes
								if (StageActor->IsUVActor() != bIsInUVMode)
								{
									continue;
								}

								WorldPosition = StageActor->GetStageActorTransform().GetTranslation();
							}
							else
							{
								WorldPosition = Actor->GetActorLocation();
							}

							FRCPreviewRenderCompletedEventActorPosition ActorInfo;
							ActorInfo.Path = Actor->GetPathName();

							PerRendererData->GetLightCardHelper().WorldToPixel(
								View,
								WorldPosition,
								ActorInfo.Position,
								ProjectionType // Pass this since the helper's projection mode made have changed since we kicked off the render
							);

							// Normalize to the size of the texture
							ActorInfo.Position /= Resolution;

							Event.ActorPositions.Add(ActorInfo);
						}
					}
				}
			}

			// Send data back to the client
			TArray<uint8> Payload;
			WebRemoteControlUtils::SerializeMessage(Event, Payload);
			RemoteControlModule->SendWebsocketMessage(ClientId, Payload);
		}
	));
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorDragBegin(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewActorDragBeginBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FPerRendererData* PerRendererData = GetClientPerRendererData(WebSocketMessage.ClientId, Body.RendererId);
	if (!PerRendererData)
	{
		return;
	}

	if (!PerRendererData->DraggedActors.IsEmpty())
	{
		// A drag is already in progress
		return;
	}

#if WITH_EDITOR
	PerRendererData->TransactionId.Invalidate();
	if (GEditor && GEditor->Trans)
	{
		if (GEditor->BeginTransaction(LOCTEXT("DragActorsTransaction", "Drag Actors with Stage App")) != INDEX_NONE)
		{
			const FTransaction* Transaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - 1);
			if (ensure(Transaction))
			{
				PerRendererData->TransactionId = Transaction->GetId();
			}
		}
	}
#endif

	PerRendererData->UpdateDragSequenceNumber(Body.SequenceNumber);
	PerRendererData->bHasDragMovedRecently = true;

	for (const FString& ActorPath : Body.Actors)
	{
		if (AActor* Actor = FindObject<AActor>(nullptr, *ActorPath))
		{
			if (!Actor || !Actor->Implements<UDisplayClusterStageActor>())
			{
				continue;
			}

			PerRendererData->DraggedActors.Emplace(FDisplayClusterWeakStageActorPtr(Actor));
		}
	}
	
	if (!Body.PrimaryActor.IsEmpty())
	{
		AActor* PrimaryActor = FindObject<AActor>(nullptr, *Body.PrimaryActor);

		if (!PrimaryActor || !PrimaryActor->Implements<UDisplayClusterStageActor>())
		{
			return;
		}

		PerRendererData->PrimaryActor = FDisplayClusterWeakStageActorPtr(PrimaryActor);
	}
	else
	{
		PerRendererData->PrimaryActor.Reset();
	}

	if (!DragTimeoutTickerHandle.IsValid())
	{
		DragTimeoutTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateRaw(this, &FStageAppRouteHandler::TimeOutDrags),
			CVarStageAppDragTimeoutCheckInterval.GetValueOnGameThread()
		);
	}
}

#if WITH_EDITOR
void FStageAppRouteHandler::OnTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState)
{
	if (InTransactionState != ETransactionStateEventType::TransactionCanceled && InTransactionState != ETransactionStateEventType::TransactionFinalized)
	{
		return;
	}

	for (TPair<FGuid, TClientIdToPerRendererDataMap>& ClientPair : PerRendererDataMapsByClientId)
	{
		for (TClientIdToPerRendererDataMap::ElementType& PerRendererDataPair : ClientPair.Value)
		{
			FPerRendererData& PerRendererData = PerRendererDataPair.Value;
			if (PerRendererData.TransactionId == InTransactionContext.TransactionId)
			{
				// Invalidate the transaction ID first so we don't try to end it a second time in EndActorDrag
				PerRendererData.TransactionId.Invalidate();
				EndActorDrag(PerRendererData, ClientPair.Key, PerRendererDataPair.Key, false);
			}
		}
	}
}
#endif

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorDragMove(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewActorDragMoveBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FPerRendererData* PerRendererData = GetClientPerRendererData(WebSocketMessage.ClientId, Body.RendererId);
	if (!PerRendererData)
	{
		return;
	}

	PerRendererData->UpdateDragSequenceNumber(Body.SequenceNumber);
	PerRendererData->bHasDragMovedRecently = true;

	DragActors(*PerRendererData, Body.DragPosition);
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorDragEnd(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewActorDragEndBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FPerRendererData* PerRendererData = GetClientPerRendererData(WebSocketMessage.ClientId, Body.RendererId);
	if (!PerRendererData)
	{
		return;
	}

	PerRendererData->UpdateDragSequenceNumber(Body.SequenceNumber);
	DragActors(*PerRendererData, Body.DragPosition);
	EndActorDrag(*PerRendererData, WebSocketMessage.ClientId, Body.RendererId, true);
}

void FStageAppRouteHandler::HandleWebSocketNDisplayPreviewActorCreate(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayPreviewActorCreateBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FPerRendererData* PerRendererData = GetClientPerRendererData(WebSocketMessage.ClientId, Body.RendererId);
	if (!PerRendererData)
	{
		return;
	}

	ADisplayClusterRootActor* RootActor = PerRendererData->GetRootActor();
	if (!RootActor || !RootActor->GetWorld())
	{
		return;
	}

	ULevel* Level = RootActor->GetWorld()->GetCurrentLevel();
	if (!Level)
	{
		return;
	}

	UClass* ActorClass = StaticLoadClass(UObject::StaticClass(), nullptr, *Body.ActorClass);
	if (!ActorClass)
	{
		return;
	}

	const UDisplayClusterStageActorTemplate* Template = nullptr;

	if (!Body.TemplatePath.IsEmpty())
	{
		if (Body.TemplatePath != TEXT("None"))
		{
			Template = Cast<UDisplayClusterStageActorTemplate>(StaticLoadObject(UDisplayClusterStageActorTemplate::StaticClass(), nullptr, *Body.TemplatePath));
		}
	}
#if WITH_EDITOR
	else if (ActorClass == ADisplayClusterLightCardActor::StaticClass())
	{
		// Use the default lightcard template if available
		Template = IDisplayClusterLightCardEditor::Get().GetDefaultLightCardTemplate();
	}
#endif

	FDisplayClusterLightCardEditorHelper::FSpawnActorArgs SpawnArgs;
	{
		SpawnArgs.ActorClass = ActorClass;
		SpawnArgs.ActorName = FName(Body.ActorName);
		SpawnArgs.RootActor = RootActor;
		SpawnArgs.Template = Template;
		SpawnArgs.Level = Level;
		SpawnArgs.ProjectionMode = PerRendererData->GetLightCardHelper().GetProjectionMode();
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("SpawnActorTransaction", "Spawn Actor from Stage App"));
#endif

	AActor* NewActor = FDisplayClusterLightCardEditorHelper::SpawnStageActor(SpawnArgs);
	if (!NewActor)
	{
		return;
	}

	if (ADisplayClusterLightCardActor* LightCard = Cast<ADisplayClusterLightCardActor>(NewActor))
	{
		if (Body.OverrideColor)
		{
			LightCard->Color = Body.Color;
		}

		LightCard->SetIsLightCardFlag(Body.bIsLightCardFlag);
	}

	// Override the location
	if (Body.OverridePosition && NewActor->Implements<UDisplayClusterStageActor>())
	{
		const FIntPoint& Resolution = PerRendererData->GetPreviewSettings().Resolution;

		const FIntPoint PixelPos(
			FMath::RoundToInt(Body.Position.X * Resolution.X),
			FMath::RoundToInt(Body.Position.Y * Resolution.Y)
		);

		FDisplayClusterLightCardEditorHelper& LightCardHelper = PerRendererData->GetLightCardHelper();

		FSceneViewInitOptions ViewInitOptions;
		if (PerRendererData->GetSceneViewInitOptions(ViewInitOptions))
		{
			const FSceneView View(ViewInitOptions);
			LightCardHelper.MoveActorsToPixel({ NewActor }, PixelPos, View);
		}
	}

	// Send new actor back to the client
	FRCRequestedActorsCreated Event;
	Event.RequestId = Body.RequestId;
	Event.ActorPaths.Add(NewActor->GetPathName());

	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(Event, Payload);
	RemoteControlModule->SendWebsocketMessage(WebSocketMessage.ClientId, Payload);
}

void FStageAppRouteHandler::HandleWebSocketStageAppActorsDuplicate(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketNDisplayActorDuplicateBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	FRCRequestedActorsCreated Event;

#if WITH_EDITOR
	TMap<UWorld*, FCachedActorLabels> ActorLabelsByWorld;
	FScopedTransaction Transaction(LOCTEXT("DuplicateActors", "Duplicate Actors from Stage App"));
#endif

	for (const FString& ActorPath : Body.Actors)
	{
		AActor* Actor = FindObject<AActor>(nullptr, *ActorPath);
		if (!Actor)
		{
			continue;
		}

		UWorld* World = Actor->GetWorld();
		if (!World)
		{
			continue;
		}

		ULevel* Level = World->GetCurrentLevel();
		if (!Level)
		{
			continue;
		}

		const FName UniqueName = MakeUniqueObjectName(Level, Actor->GetClass());
		AActor* NewActor = CastChecked<AActor>(StaticDuplicateObject(Actor, Level, UniqueName));

#if WITH_EDITOR
		Level->AddLoadedActor(NewActor);

		FCachedActorLabels* ActorLabels = ActorLabelsByWorld.Find(World);
		if (!ActorLabels)
		{
			ActorLabels = &ActorLabelsByWorld.Emplace(World, World);
		}

		FActorLabelUtilities::SetActorLabelUnique(NewActor, NewActor->GetActorLabel(), ActorLabels);
		ActorLabels->Add(NewActor->GetActorLabel());
#endif

		// Special handling for lightcards
		if (ADisplayClusterLightCardActor* NewLightCard = Cast<ADisplayClusterLightCardActor>(NewActor))
		{
			// Add it to the root actor
			if (ADisplayClusterRootActor* RootActor = StageAppRouteHandlerUtils::GetRootActorForLightCard(Actor))
			{
				NewLightCard->AddToRootActor(RootActor);

#if WITH_EDITOR
				// Required so operator panel updates
				RootActor->PostEditChange();
#endif
			}
		}

#if WITH_EDITOR
		if (GIsEditor)
		{
			GEditor->BroadcastLevelActorAdded(NewActor);
		}
#endif

		Event.ActorPaths.Add(NewActor->GetPathName());
	}

	// Send new actors back to the client
	if (!Event.ActorPaths.IsEmpty())
	{
		Event.RequestId = Body.RequestId;

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(Event, Payload);
		RemoteControlModule->SendWebsocketMessage(WebSocketMessage.ClientId, Payload);
	}
}

void FStageAppRouteHandler::HandleWebSocketStageAppClientObjectCreate(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketStageAppClientObjectCreateBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	UPackage* Package = CreatePackage(TEXT("/Engine/Transient/StageAppClientObjects"));
	if (!Package)
	{
		return;
	}

	UObject* CreatedObject = NewObject<UObject>(Package, Body.ObjectClass, NAME_None, RF_Transient | RF_Public);
	if (!CreatedObject)
	{
		return;
	}

	CreatedObject->AddToRoot();
	BoundObjectsByClientId.FindOrAdd(WebSocketMessage.ClientId).Add(CreatedObject);

	// Sent response with the new object's path
	FRCRequestedClientObjectCreated Event;

	Event.RequestId = Body.RequestId;
	Event.ObjectPath = CreatedObject->GetPathName();

	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(Event, Payload);
	RemoteControlModule->SendWebsocketMessage(WebSocketMessage.ClientId, Payload);
}

void FStageAppRouteHandler::HandleWebSocketStageAppClientObjectDestroy(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketStageAppClientObjectDestroyBody Body;
	if (!WebRemoteControlUtils::DeserializeMessage(WebSocketMessage.RequestPayload, Body))
	{
		return;
	}

	if (TArray<UObject*>* ClientBoundObjects = BoundObjectsByClientId.Find(WebSocketMessage.ClientId))
	{
		for (auto BoundObjectIt = ClientBoundObjects->CreateIterator(); BoundObjectIt; ++BoundObjectIt)
		{
			UObject* BoundObject = *BoundObjectIt;
			if (!BoundObject)
			{
				BoundObjectIt.RemoveCurrentSwap();
				continue;
			}

			if (BoundObject->GetPathName() == Body.ObjectPath)
			{
				BoundObjectIt.RemoveCurrentSwap();
				BoundObject->RemoveFromRoot();
				BoundObject->ConditionalBeginDestroy();
				return;
			}
		}
	}
}

void FStageAppRouteHandler::HandleClientDisconnected(FGuid ClientId)
{
	// Destroy the client's renderers
	if (TClientIdToPerRendererDataMap* PerRendererDataMap = PerRendererDataMapsByClientId.Find(ClientId))
	{
		IDisplayClusterScenePreview& PreviewModule = IDisplayClusterScenePreview::Get();
		for (TClientIdToPerRendererDataMap::ElementType& PerRendererDataPair : *PerRendererDataMap)
		{
			EndActorDrag(PerRendererDataPair.Value, ClientId, PerRendererDataPair.Key, true);

			PreviewModule.DestroyRenderer(PerRendererDataPair.Key);
		}

		PerRendererDataMapsByClientId.Remove(ClientId);
	}

	// Destroy objects bound to the client's connection lifetime
	if (TArray<UObject*>* ClientBoundObjects = BoundObjectsByClientId.Find(ClientId))
	{
		for (UObject* BoundObject : *ClientBoundObjects)
		{
			BoundObject->RemoveFromRoot();
			BoundObject->ConditionalBeginDestroy();
		}

		BoundObjectsByClientId.Remove(ClientId);
	}
}

void FStageAppRouteHandler::ChangePreviewRendererSettings(const FGuid& ClientId, int32 RendererId, const FRCWebSocketNDisplayPreviewRendererSettings& NewSettings)
{
	FPerRendererData* PerRendererData = GetClientPerRendererData(ClientId, RendererId);
	if (!PerRendererData)
	{
		return;
	}

	PerRendererData->SetPreviewSettings(NewSettings);
}

FStageAppRouteHandler::FPerRendererData* FStageAppRouteHandler::GetClientPerRendererData(const FGuid& ClientId, int32 RendererId)
{
	TClientIdToPerRendererDataMap* PerRendererDataMap = PerRendererDataMapsByClientId.Find(ClientId);
	if (!PerRendererDataMap)
	{
		return nullptr;
	}

	return PerRendererDataMap->Find(RendererId);
}

void FStageAppRouteHandler::DragActors(FPerRendererData& PerRendererData, FVector2D DragPosition)
{
	// Set up the scene view relative to which we're dragging
	FSceneViewInitOptions ViewInitOptions;
	if (!PerRendererData.GetSceneViewInitOptions(ViewInitOptions))
	{
		return;
	}

	PerRendererData.LastDragPosition = DragPosition;

	const FSceneView View(ViewInitOptions);

	// Determine the screen pixel position to which we're dragging
	const FRCWebSocketNDisplayPreviewRendererSettings& PreviewSettings = PerRendererData.GetPreviewSettings();
	const FIntPoint& Resolution = PreviewSettings.Resolution;

	const FIntPoint PixelPos(
		FMath::RoundToInt(DragPosition.X * Resolution.X),
		FMath::RoundToInt(DragPosition.Y * Resolution.Y)
	);
	const FVector DragWidgetOffset = FVector::ZeroVector;

	FDisplayClusterLightCardEditorHelper& LightCardHelper = PerRendererData.GetLightCardHelper();

	if (LightCardHelper.GetProjectionMode() == EDisplayClusterMeshProjectionType::UV)
	{
		LightCardHelper.DragUVActors(
			PerRendererData.DraggedActors,
			PixelPos,
			View,
			DragWidgetOffset,
			EAxisList::Type::XYZ,
			PerRendererData.PrimaryActor
		);
	}
	else
	{
		LightCardHelper.DragActors(
			PerRendererData.DraggedActors,
			PixelPos,
			View,
			FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical,
			DragWidgetOffset,
			EAxisList::Type::XYZ,
			PerRendererData.PrimaryActor
		);
	}
}

bool FStageAppRouteHandler::TimeOutDrags(float DeltaTime)
{
	bool bAnyDragsInProgress = false;
	for (TPair<FGuid, TClientIdToPerRendererDataMap>& ClientPair : PerRendererDataMapsByClientId)
	{
		for (TClientIdToPerRendererDataMap::ElementType& PerRendererDataPair : ClientPair.Value)
		{
			FPerRendererData& PerRendererData = PerRendererDataPair.Value;
			if (PerRendererData.DraggedActors.IsEmpty())
			{
				continue;
			}
			
			if (!PerRendererData.bHasDragMovedRecently)
			{
				EndActorDrag(PerRendererData, ClientPair.Key, PerRendererDataPair.Key, false);
				continue;
			}

			PerRendererData.bHasDragMovedRecently = false;
			bAnyDragsInProgress = true;
		}
	}

	if (!bAnyDragsInProgress)
	{
		DragTimeoutTickerHandle.Reset();
		return false;
	}

	return true;
}

void FStageAppRouteHandler::EndActorDrag(FPerRendererData& PerRendererData, const FGuid& ClientId, int32 RendererId, bool bEndedByClient)
{
	PerRendererData.DraggedActors.Empty();
	PerRendererData.bHasDragMovedRecently = false;

#if WITH_EDITOR
	if (PerRendererData.TransactionId.IsValid())
	{
		PerRendererData.TransactionId.Invalidate(); // Invalidate this first so we don't try to re-handle the end event
		GEditor->EndTransaction();
	}
#endif

	if (!bEndedByClient)
	{
		// Notify the client that its drag in progress was cancelled
		FRCActorDragCancelled Event;
		Event.RendererId = RendererId;

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(Event, Payload);

		RemoteControlModule->SendWebsocketMessage(ClientId, Payload);
	}
}

#undef LOCTEXT_NAMESPACE
