// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "CoreMinimal.h"
#include "DisplayClusterLightCardEditorHelper.h"
#include "UObject/ObjectMacros.h"
#include "IWebRemoteControlModule.h"
#include "Misc/ITransaction.h"
#include "RemoteControlWebsocketRoute.h"
#include "StageAppRequest.h"

class ADisplayClusterRootActor;
class ADisplayClusterLightCardActor;

/** Handler for Epic Stage App websocket requests. */
class FStageAppRouteHandler
{
private:
	/** Data used for editing lightcard positions within the context of a preview. */
	struct FPerRendererData
	{
		FPerRendererData(int RendererId);

		/** The list of actors currently being dragged by this client. Empty if a drag is not in progress. */
		TArray<FDisplayClusterWeakStageActorPtr> DraggedActors;

		/** The actor to use as the origin point for the drag operation. */
		FDisplayClusterWeakStageActorPtr PrimaryActor;

		/** The last position sent by a client during a drag operation. */
		FVector2D LastDragPosition;

		/** If true, the dragged lightcards have moved since the last timeout check. */
		bool bHasDragMovedRecently = false;

		/** If true, the client is currently waiting for a render to be completed. */
		bool bIsRenderPending = false;

#if WITH_EDITOR
		/** The GUID of the current drag transaction. */
		FGuid TransactionId;
#endif		

	public:
		/** Get or create a FDisplayClusterLightCardEditorHelper for this client preview renderer. */
		FDisplayClusterLightCardEditorHelper& GetLightCardHelper();

		/** Get the preview settings to use for renders. */
		const FRCWebSocketNDisplayPreviewRendererSettings& GetPreviewSettings() const;

		/** Change the preview settings to use for renders. */
		void SetPreviewSettings(const FRCWebSocketNDisplayPreviewRendererSettings& NewSettings);

		/** Get the scene view init options to be used for preview renders based on the stored settings. */
		bool GetSceneViewInitOptions(FSceneViewInitOptions& ViewInitOptions, bool bApplyActorRotation = true);

		/** Update the sequence number if the provided one is higher than the previous. */
		void UpdateDragSequenceNumber(int64 ReceivedSequenceNumber);

		/** Get the highest sequence number received from the client during a drag operation. */
		int64 GetDragSequenceNumber() const;

		/** Get DCRA from the scene that is used to render preview. */
		ADisplayClusterRootActor* GetRootActor() const;

		/** Get the RootActorProxy that is used for rendering to the final texture.
		* 
		* This actor is used because of special rendering rules that differ from the RootActor settings in the scene,
		* which requires a separate RootActor instance to be created.
		* For example:
		* - DCRA in a scene may have settings that disable preview rendering (or even freeze it). But this app will always force this feature.
		* - An app can have custom DisplayDevice settings. However, the DCRA of the scene cannot be modified on this side.
		* - The app can have custom preview settings, such as "PreviewSceeenPercentage," to set its own image quality, etc.
		*/
		ADisplayClusterRootActor* GetRootActorProxy() const;

	private:
		/** Update LightCardHelper's settings based on the ones stored in PreviewSettings. */
		void UpdateLightCardHelperSettings();

	private:
		/** Helper used to move light cards and project their coordinates. */
		TSharedPtr<FDisplayClusterLightCardEditorHelper> LightCardHelper;

		/** Settings used for generating previews. */
		FRCWebSocketNDisplayPreviewRendererSettings PreviewSettings;

		/** The ID of the renderer with which this is associated. */
		int RendererId;

		/** Highest sequence number received from the client during a drag operation. */
		int64 SequenceNumber = 0;
	};

	/** Map from renderer IDs created by a client (keyed by ID) to per-renderer data used by the handler. */
	using TClientIdToPerRendererDataMap = TMap<int32, FPerRendererData>;

public:
	/** Registers the WebSocket routes with the provided WebRemoteControl module. */
	void RegisterRoutes(IWebRemoteControlModule& WebRemoteControl);

	/** Unregisters the WebSocket routes with the provided WebRemoteControl module. */
	void UnregisterRoutes(IWebRemoteControlModule& WebRemoteControl);

	/** Registers events that require the engine to be initialized first. */
	void RegisterEngineEvents();

	/** Unregisters events that required the engine to be initialized first. */
	void UnregisterEngineEvents();

private:
	/** Register a WebSocket route with the WebRemoteControl module. */
	void RegisterRoute(TUniquePtr<FRemoteControlWebsocketRoute> Route);

	/** Handles creating a preview renderer. */
	void HandleWebSocketNDisplayPreviewRendererCreate(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles setting the root actor of a renderer. */
	void HandleWebSocketNDisplayPreviewRendererSetRoot(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles changing renderer settings. */
	void HandleWebSocketNDisplayPreviewRendererConfigure(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles destroying a preview renderer. */
	void HandleWebSocketNDisplayPreviewRendererDestroy(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles rendering a preview. */
	void HandleWebSocketNDisplayPreviewRender(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles beginning an actor drag operation. */
	void HandleWebSocketNDisplayPreviewActorDragBegin(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles moving actors with a drag operation. */
	void HandleWebSocketNDisplayPreviewActorDragMove(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles ending an actor drag operation. */
	void HandleWebSocketNDisplayPreviewActorDragEnd(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles creating an actor. */
	void HandleWebSocketNDisplayPreviewActorCreate(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles duplicating actors. */
	void HandleWebSocketStageAppActorsDuplicate(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles creating an object whose lifetime is tied to the lifetime of the requesting client's connection. */
	void HandleWebSocketStageAppClientObjectCreate(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles destroying an object whose lifetime is tied to the lifetime of the requesting client's connection. */
	void HandleWebSocketStageAppClientObjectDestroy(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Called when a client disconnects from the WebSocket server. */
	void HandleClientDisconnected(FGuid ClientId);

#if WITH_EDITOR
	/**
	 * Called when the state of an editor transaction changes.
	 */
	void OnTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState);
#endif

	/** Change the settings for a client's preview renderer. */
	void ChangePreviewRendererSettings(const FGuid& ClientId, int32 RendererId, const FRCWebSocketNDisplayPreviewRendererSettings& NewSettings);

	/** Get the nDisplay preview rendering data for the given client and renderer ID, or null if it's an invalid client/renderer ID. */
	FPerRendererData* GetClientPerRendererData(const FGuid& ClientId, int32 RendererId);

	/** Drag actors to the specified position. */
	void DragActors(FPerRendererData& PreviewData, FVector2D DragPosition);

	/** Check if any actor drag operations have timed out, and if so, end them. */
	bool TimeOutDrags(float DeltaTime);

	/** End a client's actor drag operation. */
	void EndActorDrag(FPerRendererData& PreviewData, const FGuid& ClientId, int32 RendererId, bool bEndedByClient);

	/** Set root actor via path. */
	void SetRendererRootActorPath(const int32 RendererId, const FString& RootActorPath);

private:
	/** The module for the WebSocket server for which this is handling routes. */
	IWebRemoteControlModule* RemoteControlModule = nullptr;

	/** Routes this has registered with the WebSocket server. */
	TArray<TUniquePtr<FRemoteControlWebsocketRoute>> Routes;

	/** Map from client GUID to preview renderer data maps for that client. */
	TMap<FGuid, TClientIdToPerRendererDataMap> PerRendererDataMapsByClientId;

	/** Map from client GUID to objects bound to the lifetime of that client's connection. */
	TMap<FGuid, TArray<UObject*>> BoundObjectsByClientId;

	/** Image wrapper module used to compress preview images. */
	class IImageWrapperModule* ImageWrapperModule = nullptr;

	/** Handle for timing out drags. */
	FTSTicker::FDelegateHandle DragTimeoutTickerHandle;
};
