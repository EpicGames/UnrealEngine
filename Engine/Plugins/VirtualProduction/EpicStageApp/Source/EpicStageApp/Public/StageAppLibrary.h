// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "StageAppLibrary.generated.h"

/**
 * Generally useful Blueprint/remote functions for Epic Stage App integration.
 */
UCLASS()
class EPICSTAGEAPP_API UStageAppFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get the current semantic version for the stage app API as a formatted string. */
	UFUNCTION(BlueprintPure, Category = "Development", meta = (BlueprintThreadSafe))
	static FString GetAPIVersion();

	/** Get the port number used to access the remote control web interface for this engine instance. */
	UFUNCTION(BlueprintPure, Category = "Development", meta = (BlueprintThreadSafe))
	static int32 GetRemoteControlWebInterfacePort();
};
