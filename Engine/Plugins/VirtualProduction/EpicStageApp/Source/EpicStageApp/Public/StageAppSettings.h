// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "StageAppSettings.generated.h"

/**
 * Global settings for Epic Stage App integration.
 */
UCLASS(config = Engine)
class EPICSTAGEAPP_API UStageAppSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	//~ Begin UDeveloperSettings implementation
	virtual FName GetContainerName() const
	{
		return "Project";
	}

	virtual FName GetCategoryName() const
	{
		return "Plugins";
	}

	virtual FName GetSectionName() const
	{
		return "Epic Stage App";
	}

	virtual FText GetSectionText() const
	{
		return NSLOCTEXT("StageAppSettings", "StageAppSettingsSection", "Epic Stage App");
	}
	//~ End UDeveloperSettings implementation

	/**
	 * The IP endpoint to listen to for app discovery messages.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Epic Stage App")
	FString DiscoveryEndpoint = "230.0.0.2";

	/**
	 * The port to listen to for app discovery messages.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Epic Stage App")
	int32 DiscoveryPort = 6667;

	/**
	 * How many seconds to wait for app discovery messages.
	 */
	UE_DEPRECATED(5.4, "This setting is no longer used.")
	UPROPERTY(config, EditAnywhere, Category = "Epic Stage App")
	float DiscoverySocketWaitTime = 10.f;
};
