// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Holds info about a version of the stage app API. */
struct EPICSTAGEAPP_API FEpicStageAppAPIVersion
{
	/** The major component of the current version. */
	static const uint16 Major = 1;

	/** The minor component of the current version. */
	static const uint16 Minor = 10;
	
	/** The patch component of the current version. */
	static const uint16 Patch = 0;
};